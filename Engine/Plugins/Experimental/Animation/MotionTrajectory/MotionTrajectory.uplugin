{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "0.1",
	"FriendlyName": "Motion Trajectory",
	"Description": "Generate predictions and track history of character motion.",
	"Category": "Animation",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"EnabledByDefault": false,
	"CanContainContent": true,
	"IsBetaVersion": false,
	"IsExperimentalVersion": true,
	"Installed": false,
	"Modules": [
		{
			"Name": "MotionTrajectory",
			"Type": "Runtime",
			"LoadingPhase": "PreDefault"
		}
	],
	"Plugins": [
		{
			"Name": "PoseSearch",
			"Enabled": true
		}
	]
}