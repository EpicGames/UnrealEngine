// Copyright Epic Games, Inc. All Rights Reserved.

#include "CharacterTrajectoryComponent.h"

#include "Components/SkeletalMeshComponent.h"
#include "GameFramework/Character.h"
#include "HAL/IConsoleManager.h"
#include "MotionTrajectory.h"
#include "Animation/TrajectoryTypes.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CharacterTrajectoryComponent)

#if ENABLE_ANIM_DEBUG
TAutoConsoleVariable<int32> CVarCharacterTrajectoryDebug(TEXT("a.CharacterTrajectory.Debug"), 0, TEXT("Turn on debug drawing for Character trajectory"));
#endif // ENABLE_ANIM_DEBUG

UCharacterTrajectoryComponent::UCharacterTrajectoryComponent(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	PrimaryComponentTick.bCanEverTick = false;
	PrimaryComponentTick.bStartWithTickEnabled = false;

	bWantsInitializeComponent = true;
}

void UCharacterTrajectoryComponent::InitializeComponent()
{
	Super::InitializeComponent();

	if (ACharacter* Character = Cast<ACharacter>(GetOwner()))
	{
		Character->OnCharacterMovementUpdated.AddDynamic(this, &UCharacterTrajectoryComponent::OnMovementUpdated);
	}
	else
	{
		UE_LOGF(LogMotionTrajectory, Error, "UCharacterTrajectoryComponent requires its owner to be ACharacter");
	}
}

void UCharacterTrajectoryComponent::UninitializeComponent()
{
	if (ACharacter* Character = Cast<ACharacter>(GetOwner()))
	{
		Character->OnCharacterMovementUpdated.RemoveDynamic(this, &UCharacterTrajectoryComponent::OnMovementUpdated);
	}
	else
	{
		UE_LOGF(LogMotionTrajectory, Error, "UCharacterTrajectoryComponent requires its owner to be ACharacter");
	}

	Super::UninitializeComponent();
}

void UCharacterTrajectoryComponent::BeginPlay()
{
	Super::BeginPlay();

	SamplingData.Init();
	TranslationHistory.Init(FVector::ZeroVector, SamplingData.NumHistorySamples);

	if (const ACharacter* Character = Cast<ACharacter>(GetOwner()))
	{
		if (const USkeletalMeshComponent* MeshComp = Character->GetMesh())
		{
			const FVector Position = MeshComp->GetComponentLocation();
			const FQuat Facing = MeshComp->GetComponentRotation().Quaternion();

			FMotionTrajectoryLibrary::InitTrajectorySamples(Trajectory, SamplingData, Position, Facing);
		}
		else
		{
			ensure(false);
		}
	}
	else
	{
		UE_LOGF(LogMotionTrajectory, Error, "UCharacterTrajectoryComponent requires its owner to be ACharacter");
	}
}

void UCharacterTrajectoryComponent::OnMovementUpdated(float DeltaSeconds, FVector OldLocation, FVector OldVelocity)
{
	if (LastUpdateFrameNumber != 0 && LastUpdateFrameNumber == GFrameNumber)
	{
		return;
	}

	if (DeltaSeconds <= 0.f)
	{
		return;
	}

	ACharacter* Character = Cast<ACharacter>(GetOwner());
	if (!ensure(Character))
	{
		return;
	}

	CharacterTrajectoryData.UpdateDataFromCharacter(DeltaSeconds, Character);
	FMotionTrajectoryLibrary::UpdateHistory_TransformHistory(Trajectory, TranslationHistory, CharacterTrajectoryData, SamplingData, DeltaSeconds);
	FMotionTrajectoryLibrary::UpdatePrediction_SimulateCharacterMovement(Trajectory, CharacterTrajectoryData, SamplingData);

	LastUpdateFrameNumber = GFrameNumber;

#if ENABLE_ANIM_DEBUG
	if (CVarCharacterTrajectoryDebug.GetValueOnAnyThread())
	{
		UTransformTrajectoryBlueprintLibrary::DebugDrawTrajectory(Trajectory, GetWorld());
	}
#endif // ENABLE_ANIM_DEBUG
}
