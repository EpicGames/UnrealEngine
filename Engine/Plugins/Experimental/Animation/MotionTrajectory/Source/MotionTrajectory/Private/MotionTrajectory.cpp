// Copyright Epic Games, Inc. All Rights Reserved.

#include "MotionTrajectory.h"
#include "Modules/ModuleManager.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MotionTrajectory)

DEFINE_LOG_CATEGORY(LogMotionTrajectory);

void FMotionTrajectoryModule::StartupModule()
{
}

void FMotionTrajectoryModule::ShutdownModule()
{
}

IMPLEMENT_MODULE(FMotionTrajectoryModule, MotionTrajectory)
