// Copyright Epic Games, Inc. All Rights Reserved.

#include "MotionTrajectoryLibrary.h"
#include "Components/SkeletalMeshComponent.h"
#include "GameFramework/Character.h"
#include "GameFramework/CharacterMovementComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MotionTrajectoryLibrary)

void FTrajectorySamplingData::Init()
{
	// The UI clamps these to be non-zero.
	check(HistorySamplesPerSecond);
	check(PredictionSamplesPerSecond);

	NumHistorySamples = FMath::CeilToInt32(HistoryLengthSeconds * HistorySamplesPerSecond);
	SecondsPerHistorySample = 1.f / HistorySamplesPerSecond;

	NumPredictionSamples = FMath::CeilToInt32(PredictionLengthSeconds * PredictionSamplesPerSecond);
	SecondsPerPredictionSample = 1.f / PredictionSamplesPerSecond;
}

void FCharacterTrajectoryData::UpdateDataFromCharacter(float DeltaSeconds, const ACharacter* Character)
{
	if (!ensure(Character))
	{
		return;
	}

	// An AnimInstance might call this during an AnimBP recompile with 0 delta time.
	if (DeltaSeconds <= 0.f)
	{
		return;
	}

	if (const UCharacterMovementComponent* MoveComp = Character->GetCharacterMovement())
	{
		MaxSpeed = FMath::Max(MoveComp->GetMaxSpeed() * MoveComp->GetAnalogInputModifier(), MoveComp->GetMinAnalogSpeed());
		BrakingDeceleration = FMath::Max(0.f, MoveComp->GetMaxBrakingDeceleration());
		bOrientRotationToMovement = MoveComp->bOrientRotationToMovement;

		Velocity = MoveComp->Velocity;
		Acceleration = MoveComp->GetCurrentAcceleration();

		if (Acceleration.IsZero())
		{
			Friction = MoveComp->bUseSeparateBrakingFriction ? MoveComp->BrakingFriction : MoveComp->GroundFriction;
			const float FrictionFactor = FMath::Max(0.f, MoveComp->BrakingFrictionFactor);
			Friction = FMath::Max(0.f, Friction * FrictionFactor);
		}
		else
		{
			Friction = MoveComp->GroundFriction;
		}
	}
	else
	{
		ensure(false);
	}

	// @todo: Simulated proxies don't have controllers, so they'll need some other mechanism to account for controller rotation rate.
	const AController* Controller = Character->GetController();
	if (Controller)
	{
		const FQuat CurrentControllerRotation = Controller->GetDesiredRotation().Quaternion();

		if (bEnableNonStandardGravity)
		{
			const FVector CharacterUp = Facing.GetUpVector();

			// Compute the controller's rotation delta in world space, then extract just the twist around the character's up vector.
			// Swing-twist decomposition isolates yaw (rotation around CharacterUp) from pitch/roll changes.
			FQuat DeltaQuat = CurrentControllerRotation * DesiredControllerRotationLastUpdate.Inverse();
			if (DeltaQuat.W < 0.f)
			{
				DeltaQuat = -DeltaQuat; // keep W >= 0 so ToAxisAndAngle returns the short-path angle in [0, 180].
			}

			const float TwistDot = FVector::DotProduct(FVector(DeltaQuat.X, DeltaQuat.Y, DeltaQuat.Z), CharacterUp);
			FQuat Twist(CharacterUp.X * TwistDot, CharacterUp.Y * TwistDot, CharacterUp.Z * TwistDot, DeltaQuat.W);
			// Guard against NaN: if DeltaQuat is a 180-degree rotation exactly perpendicular to CharacterUp,
			// Very unlikely to happen as it would require a 180 degree pitch / roll rotation in one frame.
			if (Twist.SizeSquared() < UE_SMALL_NUMBER)
			{
				ControllerYawRate = 0.f;
			}
			else
			{
				Twist.Normalize();

				FVector TwistAxis;
				float TwistAngleRad;
				Twist.ToAxisAndAngle(TwistAxis, TwistAngleRad);

				ControllerYawRate = FMath::RadiansToDegrees(TwistAngleRad) * FVector::DotProduct(TwistAxis, CharacterUp) * (1.f / DeltaSeconds);
			}
		}
		else
		{
			const float DesiredYawDelta = FRotator::NormalizeAxis(Controller->GetDesiredRotation().Yaw - DesiredControllerYawLastUpdate);
			ControllerYawRate = DesiredYawDelta * (1.f / DeltaSeconds);
		}

		DesiredControllerYawLastUpdate = Controller->GetDesiredRotation().Yaw;
		DesiredControllerRotationLastUpdate = CurrentControllerRotation;

		ControllerYawRateClamped = ControllerYawRate;
		if (MaxControllerYawRate >= 0.f)
		{
			ControllerYawRateClamped = FMath::Sign(ControllerYawRate) * FMath::Min(FMath::Abs(ControllerYawRate), MaxControllerYawRate);
		}
	}

	if (const USkeletalMeshComponent* MeshComp = Character->GetMesh())
	{
		Position = MeshComp->GetComponentLocation();
		Facing = MeshComp->GetComponentRotation().Quaternion();
		MeshCompRelativeRotation = MeshComp->GetRelativeRotation().Quaternion();
	}
	else
	{
		ensure(false);
	}
}

FVector FCharacterTrajectoryData::StepCharacterMovementGroundPrediction(float DeltaSeconds, const FVector& InVelocity, const FVector& InAcceleration) const
{
	FVector OutVelocity = InVelocity;

	// Braking logic is copied from UCharacterMovementComponent::ApplyVelocityBraking()
	if (InAcceleration.IsZero())
	{
		if (InVelocity.IsZero())
		{
			return FVector::ZeroVector;
		}

		const bool bZeroFriction = (Friction == 0.f);
		const bool bZeroBraking = (BrakingDeceleration == 0.f);

		if (bZeroFriction && bZeroBraking)
		{
			return InVelocity;
		}

		static const float MaxTimeStep = 1.f / 60.f;
		float RemainingTime = DeltaSeconds;

		const FVector PrevLinearVelocity = OutVelocity;
		const FVector RevAccel = (bZeroBraking ? FVector::ZeroVector : (-BrakingDeceleration * OutVelocity.GetSafeNormal()));

		// Decelerate to brake to a stop
		while (RemainingTime >= UCharacterMovementComponent::MIN_TICK_TIME)
		{
			// Zero friction uses constant deceleration, so no need for iteration.
			const float dt = ((RemainingTime > MaxTimeStep && !bZeroFriction) ? FMath::Min(MaxTimeStep, RemainingTime * 0.5f) : RemainingTime);
			RemainingTime -= dt;

			// apply friction and braking
			OutVelocity = OutVelocity + ((-Friction) * OutVelocity + RevAccel) * dt;

			// Don't reverse direction
			if ((OutVelocity | PrevLinearVelocity) <= 0.f)
			{
				OutVelocity = FVector::ZeroVector;
				return OutVelocity;
			}
		}

		// Clamp to zero if nearly zero, or if below min threshold and braking
		const float VSizeSq = OutVelocity.SizeSquared();
		if (VSizeSq <= KINDA_SMALL_NUMBER || (!bZeroBraking && VSizeSq <= FMath::Square(UCharacterMovementComponent::BRAKE_TO_STOP_VELOCITY)))
		{
			OutVelocity = FVector::ZeroVector;
		}
	}
	// Acceleration logic is copied from  UCharacterMovementComponent::CalcVelocity
	else
	{
		const FVector AccelDir = InAcceleration.GetSafeNormal();
		const float VelSize = OutVelocity.Size();

		OutVelocity = OutVelocity - (OutVelocity - AccelDir * VelSize) * FMath::Min(DeltaSeconds * Friction, 1.f);

		OutVelocity += InAcceleration * DeltaSeconds;
		OutVelocity = OutVelocity.GetClampedToMaxSize(MaxSpeed);
	}

	return OutVelocity;
}

void FMotionTrajectoryLibrary::InitTrajectorySamples(FTransformTrajectory& Trajectory,
	const FTrajectorySamplingData& SamplingData, const FVector& Position, const FQuat& Facing)
{
	// History + current sample + prediction
	Trajectory.Samples.SetNumUninitialized(SamplingData.NumHistorySamples + 1 + SamplingData.NumPredictionSamples);

	// Initialize history samples
	for (int32 i = 0; i < SamplingData.NumHistorySamples; ++i)
	{
		Trajectory.Samples[i].Position = Position;
		Trajectory.Samples[i].Facing = Facing;
		Trajectory.Samples[i].TimeInSeconds = SamplingData.SecondsPerHistorySample * (i - SamplingData.NumHistorySamples);
	}

	// Initialize current sample and prediction
	for (int32 i = SamplingData.NumHistorySamples; i < Trajectory.Samples.Num(); ++i)
	{
		Trajectory.Samples[i].Position = Position;
		Trajectory.Samples[i].Facing = Facing;
		Trajectory.Samples[i].TimeInSeconds = SamplingData.SecondsPerPredictionSample * (i - SamplingData.NumHistorySamples);
	}
}

void FMotionTrajectoryLibrary::UpdateHistory_TransformHistory(FTransformTrajectory& Trajectory, TArrayView<FVector> TranslationHistory,
	const FCharacterTrajectoryData& CharacterTrajectoryData, const FTrajectorySamplingData& SamplingData, float DeltaSeconds)
{
	check(SamplingData.NumHistorySamples <= Trajectory.Samples.Num());
	check(TranslationHistory.Num() == SamplingData.NumHistorySamples);

	FVector CurrentTranslation = CharacterTrajectoryData.Velocity * DeltaSeconds;

	// Shift history Samples when it's time to record a new one.
	if (SamplingData.NumHistorySamples > 0 && FMath::Abs(Trajectory.Samples[SamplingData.NumHistorySamples - 1].TimeInSeconds) >= SamplingData.SecondsPerHistorySample)
	{
		for (int32 Index = 0; Index < SamplingData.NumHistorySamples - 1; ++Index)
		{
			Trajectory.Samples[Index].TimeInSeconds = Trajectory.Samples[Index + 1].TimeInSeconds;
			TranslationHistory[Index] = TranslationHistory[Index + 1] + CurrentTranslation;
			Trajectory.Samples[Index].Facing = Trajectory.Samples[Index + 1].Facing;
		}

		Trajectory.Samples[SamplingData.NumHistorySamples - 1].TimeInSeconds = 0.f;
		TranslationHistory[SamplingData.NumHistorySamples - 1] = CurrentTranslation;
		Trajectory.Samples[SamplingData.NumHistorySamples - 1].Facing = CharacterTrajectoryData.Facing;
	}
	else
	{
		for (int32 Index = 0; Index < SamplingData.NumHistorySamples; ++Index)
		{
			TranslationHistory[Index] += CurrentTranslation;
		}
	}

	// Update trajectory samples by applying the tracked translations to the current world position.
	for (int32 Index = 0; Index < SamplingData.NumHistorySamples; ++Index)
	{
		Trajectory.Samples[Index].TimeInSeconds -= DeltaSeconds;
		Trajectory.Samples[Index].Position = CharacterTrajectoryData.Position - TranslationHistory[Index];
	}
}

FVector FMotionTrajectoryLibrary::RemapVectorMagnitudeWithCurve(const FVector& Vector, bool bUseCurve, const FRuntimeFloatCurve& Curve)
{
	if (bUseCurve)
	{
		const float Length = Vector.Length();
		if (Length > UE_KINDA_SMALL_NUMBER)
		{
			const float RemappedLength = Curve.GetRichCurveConst()->Eval(Length);
			return Vector * (RemappedLength / Length);
		}
	}

	return Vector;
}

void FMotionTrajectoryLibrary::UpdatePrediction_SimulateCharacterMovement(FTransformTrajectory& Trajectory,
	const FCharacterTrajectoryData& CharacterTrajectoryData, const FTrajectorySamplingData& SamplingData)
{
	FVector CurrentPositionWS = CharacterTrajectoryData.Position;
	FVector CurrentVelocityWS = FMotionTrajectoryLibrary::RemapVectorMagnitudeWithCurve(CharacterTrajectoryData.Velocity,
		CharacterTrajectoryData.bUseSpeedRemappingCurve, CharacterTrajectoryData.SpeedRemappingCurve);
	FVector CurrentAccelerationWS = FMotionTrajectoryLibrary::RemapVectorMagnitudeWithCurve(CharacterTrajectoryData.Acceleration,
		CharacterTrajectoryData.bUseAccelerationRemappingCurve, CharacterTrajectoryData.AccelerationRemappingCurve);

	// bending CurrentVelocityWS towards CurrentAccelerationWS
	if (CharacterTrajectoryData.BendVelocityTowardsAcceleration > UE_KINDA_SMALL_NUMBER && !CurrentAccelerationWS.IsNearlyZero())
	{
		const float CurrentSpeed = CurrentVelocityWS.Length();
		const FVector VelocityWSAlongAcceleration = CurrentAccelerationWS.GetUnsafeNormal() * CurrentSpeed;
		if (CharacterTrajectoryData.BendVelocityTowardsAcceleration < 1.f - UE_KINDA_SMALL_NUMBER)
		{
			CurrentVelocityWS = FMath::Lerp(CurrentVelocityWS, VelocityWSAlongAcceleration, CharacterTrajectoryData.BendVelocityTowardsAcceleration);

			const float NewLength = CurrentVelocityWS.Length();
			if (NewLength > UE_KINDA_SMALL_NUMBER)
			{
				CurrentVelocityWS *= CurrentSpeed / NewLength;
			}
			else
			{
				// @todo: consider setting the CurrentVelocityWS = VelocityWSAlongAcceleration if vel and acc are in opposite directions
			}
		}
		else
		{
			CurrentVelocityWS = VelocityWSAlongAcceleration;
		}
	}

	FQuat CurrentFacingWS = CharacterTrajectoryData.Facing;
	FQuat SkelMeshCompRelativeRotation = CharacterTrajectoryData.MeshCompRelativeRotation;

	const FVector RotationAxis = CharacterTrajectoryData.bEnableNonStandardGravity ? CharacterTrajectoryData.Facing.GetUpVector() : FVector::UpVector;
	const FQuat ControllerRotationPerStep = FQuat(RotationAxis, FMath::DegreesToRadians(CharacterTrajectoryData.ControllerYawRateClamped * SamplingData.SecondsPerPredictionSample));

	float AccumulatedSeconds = 0.f;

	const int32 LastIndex = Trajectory.Samples.Num() - 1;
	if (SamplingData.NumHistorySamples <= LastIndex)
	{
		for (int32 Index = SamplingData.NumHistorySamples; ; ++Index)
		{
			Trajectory.Samples[Index].Position = CurrentPositionWS;
			Trajectory.Samples[Index].Facing = CurrentFacingWS;
			Trajectory.Samples[Index].TimeInSeconds = AccumulatedSeconds;

			if (Index == LastIndex)
			{
				break;
			}

			CurrentPositionWS += CurrentVelocityWS * SamplingData.SecondsPerPredictionSample;
			AccumulatedSeconds += SamplingData.SecondsPerPredictionSample;

			// Account for the controller (e.g. the camera) rotating.
			CurrentFacingWS = ControllerRotationPerStep * CurrentFacingWS;
			CurrentAccelerationWS = FMotionTrajectoryLibrary::RemapVectorMagnitudeWithCurve(ControllerRotationPerStep * CurrentAccelerationWS,
				CharacterTrajectoryData.bUseAccelerationRemappingCurve, CharacterTrajectoryData.AccelerationRemappingCurve);

			FVector NewVelocityWS = CharacterTrajectoryData.StepCharacterMovementGroundPrediction(SamplingData.SecondsPerPredictionSample, CurrentVelocityWS, CurrentAccelerationWS);

			CurrentVelocityWS = FMotionTrajectoryLibrary::RemapVectorMagnitudeWithCurve(NewVelocityWS,
				CharacterTrajectoryData.bUseSpeedRemappingCurve, CharacterTrajectoryData.SpeedRemappingCurve);

			if (CharacterTrajectoryData.bOrientRotationToMovement && !CurrentAccelerationWS.IsNearlyZero())
			{
				// Rotate towards acceleration.
				const FVector CurrentAccelerationCS = SkelMeshCompRelativeRotation.RotateVector(CurrentAccelerationWS);
				CurrentFacingWS = FMath::QInterpConstantTo(CurrentFacingWS, CurrentAccelerationCS.ToOrientationQuat(), SamplingData.SecondsPerPredictionSample,
					CharacterTrajectoryData.RotateTowardsMovementSpeed);
			}
		}
	}
}
