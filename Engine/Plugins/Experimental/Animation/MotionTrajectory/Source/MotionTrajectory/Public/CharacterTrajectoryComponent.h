// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Components/ActorComponent.h"
#include "MotionTrajectoryLibrary.h"
#include "Animation/TrajectoryTypes.h"
#include "CharacterTrajectoryComponent.generated.h"

#define UE_API MOTIONTRAJECTORY_API

// Component for generating trajectories usable by Motion Matching. This component generates trajectories from ACharacter.
// This is intended to provide an example and starting point for using Motion Matching with a common setup using the default UCharacterMovementComponent.
// It is expected work flow to extend or replace this component for projects that use a custom movement component or custom movement modes.
UCLASS(MinimalAPI, Blueprintable, BlueprintType, meta = (BlueprintSpawnableComponent), Experimental)
class UCharacterTrajectoryComponent : public UActorComponent
{
	GENERATED_BODY()

public:
	
	UE_API UCharacterTrajectoryComponent(const FObjectInitializer& ObjectInitializer);

	// Begin UActorComponent Interface
	UE_API virtual void InitializeComponent() override;
	UE_API virtual void UninitializeComponent() override;
	UE_API virtual void BeginPlay() override;
	// End UActorComponent Interface

protected:
	UFUNCTION()
	UE_API void OnMovementUpdated(float DeltaSeconds, FVector OldLocation, FVector OldVelocity);

	// Trajectory stored in world space so it can be directly passed to Motion Matching.
	UPROPERTY(Transient, BlueprintReadOnly, Category = "Trajectory")
	FTransformTrajectory Trajectory;

	UPROPERTY(Transient, BlueprintReadOnly, Category = "Trajectory Settings")
	FTrajectorySamplingData SamplingData;

	UPROPERTY(Transient, BlueprintReadOnly, Category = "Trajectory Settings")
	FCharacterTrajectoryData CharacterTrajectoryData;

	TArray<FVector> TranslationHistory;

	uint32 LastUpdateFrameNumber = 0;
};

#undef UE_API
