// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Components/ActorComponent.h"
#include "Containers/RingBuffer.h"

#include "Modules/ModuleInterface.h"
#include "MotionTrajectory.generated.h"

DECLARE_LOG_CATEGORY_EXTERN(LogMotionTrajectory, Log, All);

class APawn;

class FMotionTrajectoryModule : public IModuleInterface
{
public: 

	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
};

// Specifies the chosen domain parameters for trajectory sample retention and creation
USTRUCT(BlueprintType, Category="Motion Trajectory")
struct FMotionTrajectorySettings
{
	GENERATED_BODY()

	// Sample time horizon
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Settings", meta=(ClampMin="0.0"))
	float Seconds = 2.f;
};
