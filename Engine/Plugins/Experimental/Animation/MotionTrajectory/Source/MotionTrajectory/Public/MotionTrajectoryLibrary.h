// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Curves/CurveFloat.h"
#include "Animation/TrajectoryTypes.h"
#include "MotionTrajectoryLibrary.generated.h"

#define UE_API MOTIONTRAJECTORY_API

class ACharacter;
class UCharacterMovementComponent;

USTRUCT(BlueprintType)
struct FTrajectorySamplingData
{
	GENERATED_BODY()

	UE_API void Init();

	// This should generally match the longest history required by a Motion Matching Database in the project.
	// Motion Matching will use extrapolation to generate samples if the history doesn't contain enough samples.
	UPROPERTY(EditDefaultsOnly, BlueprintReadOnly, Category = "Trajectory Settings")
	float HistoryLengthSeconds = 1.5f;

	// Higher values will cost more storage and processing time, but give higher accuracy.
	UPROPERTY(EditDefaultsOnly, BlueprintReadOnly, Category = "Trajectory Settings", meta = (ClampMin = "1", ClampMax = "120"))
	int32 HistorySamplesPerSecond = 5;

	// This should match the longest trajectory prediction required by a Motion Matching Database in the project.
	// Motion Matching will use extrapolation to generate samples if the prediction doesn't contain enough samples.
	UPROPERTY(EditDefaultsOnly, BlueprintReadOnly, Category = "Trajectory Settings")
	float PredictionLengthSeconds = 1.5f;

	// Higher values will cost more storage and processing time, but give higher accuracy.
	UPROPERTY(EditDefaultsOnly, BlueprintReadOnly, Category = "Trajectory Settings", meta = (ClampMin = "1", ClampMax = "120"))
	int32 PredictionSamplesPerSecond = 5;

	int32 NumHistorySamples = 0;
	float SecondsPerHistorySample = 0.f;

	int32 NumPredictionSamples = 0;
	float SecondsPerPredictionSample = 0.f;
};

USTRUCT(BlueprintType)
struct FCharacterTrajectoryData
{
public:
	GENERATED_BODY()

	UE_API void UpdateDataFromCharacter(float DeltaSeconds, const ACharacter* Character);

	UE_API FVector StepCharacterMovementGroundPrediction(float DeltaSeconds, const FVector& InVelocity, const FVector& InAcceleration) const;

	// If the character is forward facing (i.e. bOrientRotationToMovement is true), this controls how quickly the trajectory will rotate
	// to face acceleration. It's common for this to differ from the rotation rate of the character, because animations are often authored 
	// with different rotation speeds than the character. This is especially true in cases where the character rotation snaps to movement.
	UPROPERTY(EditDefaultsOnly, Category = "Trajectory Settings")
	float RotateTowardsMovementSpeed = 10.f;

	// Maximum controller yaw  rate in degrees per second used to clamp the character owner controller desired yaw to generate the prediction trajectory.
	// Negative values disable the clamping behavior
	UPROPERTY(EditDefaultsOnly, Category = "Trajectory Settings")
	float MaxControllerYawRate = 70.f;

	// artificially bend character velocity towards acceleration direction to compute trajectory prediction, to get sharper turns
	// 0: character velocity is used with no alteration, 1: the acceleration direction is used as velocity direction
	UPROPERTY(EditDefaultsOnly, Category = "Trajectory Settings", meta = (ClampMin = "0", ClampMax = "1"))
	float BendVelocityTowardsAcceleration = 0.f;

	UPROPERTY(EditDefaultsOnly, Category = "Trajectory Settings", meta = (InlineEditConditionToggle))
	bool bUseSpeedRemappingCurve = false;

	UPROPERTY(EditDefaultsOnly, Category = "Trajectory Settings", meta = (EditCondition = "bUseSpeedRemappingCurve"))
	FRuntimeFloatCurve SpeedRemappingCurve;

	UPROPERTY(EditDefaultsOnly, Category = "Trajectory Settings", meta = (InlineEditConditionToggle))
	bool bUseAccelerationRemappingCurve = false;

	UPROPERTY(EditDefaultsOnly, Category = "Trajectory Settings", meta = (EditCondition = "bUseAccelerationRemappingCurve"))
	FRuntimeFloatCurve AccelerationRemappingCurve;

	// When enabled, controller yaw rate is computed as rotation around the character's up vector instead of world Z.
	// Required for correct trajectory prediction on non-standard gravity surfaces (walls, ceilings, slopes).
	UPROPERTY(EditDefaultsOnly, Category = "Trajectory Settings")
	bool bEnableNonStandardGravity = false;

	float ControllerYawRate = 0.f;
	float ControllerYawRateClamped = 0.f;
	float DesiredControllerYawLastUpdate = 0.f;
	FQuat DesiredControllerRotationLastUpdate = FQuat::Identity;

	float MaxSpeed = 0.f;
	float BrakingDeceleration = 0.f;
	float Friction = 0.f;

	FVector Velocity = FVector::ZeroVector;
	FVector Acceleration = FVector::ZeroVector;

	FVector Position = FVector::ZeroVector;
	FQuat Facing = FQuat::Identity;
	FQuat MeshCompRelativeRotation = FQuat::Identity;
	bool bOrientRotationToMovement = false;
};

/**
 * Set of functions to help populate a FPoseSearchQueryTrajectory for motion matching.
 * UCharacterTrajectoryComponent uses these functions, but they can also be used by a UAnimInstance to avoid the component.
 */
struct FMotionTrajectoryLibrary
{
public:
	static UE_API void InitTrajectorySamples(FTransformTrajectory& Trajectory,
		const FTrajectorySamplingData& SamplingData, const FVector& Position, const FQuat& Facing);

	// Update history by tracking offsets that result from character intent (e.g. movement component velocity) and applying
	// that to the current world transform. This works well on moving platforms as it only stores a history of movement
	// that results from character intent, not movement from platforms.
	static UE_API void UpdateHistory_TransformHistory(FTransformTrajectory& Trajectory, TArrayView<FVector> TranslationHistory,
		const FCharacterTrajectoryData& CharacterTrajectoryData, const FTrajectorySamplingData& SamplingData, float DeltaSeconds);

	// Update prediction by simulating the movement math for ground locomotion from UCharacterMovementComponent.
	static UE_API void UpdatePrediction_SimulateCharacterMovement(FTransformTrajectory& Trajectory,
		const FCharacterTrajectoryData& CharacterTrajectoryData, const FTrajectorySamplingData& SamplingData);

private:
	static FVector RemapVectorMagnitudeWithCurve(const FVector& Vector, bool bUseCurve, const FRuntimeFloatCurve& Curve);
};

#undef UE_API
